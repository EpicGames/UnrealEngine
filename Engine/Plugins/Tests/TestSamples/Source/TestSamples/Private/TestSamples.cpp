// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestSamples.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FTestSamplesModule"

DEFINE_LOG_CATEGORY_STATIC(LogTestSamples, Log, All);

void FTestSamplesModule::StartupModule()
{
	// Hook callbacks on before all tests Start and End event
	FAutomationTestFramework::Get().OnBeforeAllTestsEvent.AddRaw(this, &FTestSamplesModule::OnBeforeAllTests);
	FAutomationTestFramework::Get().OnAfterAllTestsEvent.AddRaw(this, &FTestSamplesModule::OnAfterAllTests);

	// Hook callbacks to Test Start and End event
	FAutomationTestFramework::Get().OnTestStartEvent.AddRaw(this, &FTestSamplesModule::OnTestStart);
	FAutomationTestFramework::Get().OnTestEndEvent.AddRaw(this, &FTestSamplesModule::OnTestEnd);

	// Hook callbacks to Test Section Enter and Leave event
	FAutomationTestFramework::Get().GetOnEnteringTestSection(TEXT("TestFramework")).AddRaw(this, &FTestSamplesModule::OnEnteringTestSection);
	FAutomationTestFramework::Get().GetOnLeavingTestSection(TEXT("TestFramework")).AddRaw(this, &FTestSamplesModule::OnLeavingTestSection);
}

void FTestSamplesModule::ShutdownModule()
{
	// Remove all registered callbacks on shutdown
	FAutomationTestFramework::Get().OnTestStartEvent.RemoveAll(this);
	FAutomationTestFramework::Get().OnTestEndEvent.RemoveAll(this);

	FAutomationTestFramework::Get().OnBeforeAllTestsEvent.RemoveAll(this);
	FAutomationTestFramework::Get().OnAfterAllTestsEvent.RemoveAll(this);

	FAutomationTestFramework::Get().GetOnEnteringTestSection(TEXT("TestFramework")).RemoveAll(this);
	FAutomationTestFramework::Get().GetOnLeavingTestSection(TEXT("TestFramework")).RemoveAll(this);
}

void FTestSamplesModule::OnTestStart(FAutomationTestBase* Test)
{
	if (Test != nullptr)
	{
		UE_LOGF(LogTestSamples, Verbose, "Starting test: %ls", *Test->GetTestFullName())
	}
}
void FTestSamplesModule::OnTestEnd(FAutomationTestBase* Test)
{
	if (Test != nullptr)
	{
		UE_LOGF(LogTestSamples, Verbose, "Ending test: %ls", *Test->GetTestFullName())
	}

}

void FTestSamplesModule::OnBeforeAllTests()
{
	UE_LOGF(LogTestSamples, Verbose, "Start running tests");
}

void FTestSamplesModule::OnAfterAllTests()
{
	UE_LOGF(LogTestSamples, Verbose, "Running tests completed");
}

void FTestSamplesModule::OnEnteringTestSection(const FString& Section)
{
	UE_LOGF(LogTestSamples, Verbose, "Entering section %ls", *Section);
}

void FTestSamplesModule::OnLeavingTestSection(const FString& Section)
{
	UE_LOGF(LogTestSamples, Verbose, "Leaving section %ls", *Section);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FTestSamplesModule, TestSamples)
