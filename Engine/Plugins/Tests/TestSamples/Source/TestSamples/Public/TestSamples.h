// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Misc/AutomationTest.h"

class FTestSamplesModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void OnBeforeAllTests();
	void OnAfterAllTests();

	void OnTestStart(FAutomationTestBase* Test);
	void OnTestEnd(FAutomationTestBase* Test);

	void OnEnteringTestSection(const FString& Section);
	void OnLeavingTestSection(const FString& Section);
};
